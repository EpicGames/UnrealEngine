// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class ConfigFileTests
	{
		#region -- Test Context --

		private const string TEST_FILE_NAME = "ConfigFileTest.ini";

		private class TestConfigFile : IDisposable
		{
			internal string? TemporaryFile { get; private set; }

			internal TestConfigFile(string fileName, string contents)
			{
				try
				{
					TemporaryFile = Path.Combine(System.IO.Path.GetTempPath(), Guid.NewGuid().ToString() + "_" + fileName);
					using (StreamWriter writer = new(TemporaryFile))
					{
						writer.Write(contents);
					}
				}
				catch (Exception)
				{
					TemporaryFile = null;
				}
			}

			public void Dispose()
			{
				if (!String.IsNullOrEmpty(TemporaryFile) && File.Exists(TemporaryFile))
				{
					File.Delete(TemporaryFile);
				}
			}
		}

		#endregion

		[TestMethod]
		public void ConfigFileTestsQuotations()
		{
			const string TestSection = "MySection";
			const string TestValueKey = "Name";
			const string TestValue = "\"https://www.unrealengine.com/\"";

			string testContents = String.Format("[{0}]\r\n{1}={2}", TestSection, TestValueKey, TestValue);

			using (TestConfigFile temporaryConfigFile = new(TEST_FILE_NAME, testContents))
			{
				string? testFilePath = temporaryConfigFile.TemporaryFile;

				Assert.IsFalse(String.IsNullOrEmpty(testFilePath), "Unable to generate a test file.");

				ConfigFile configFile;
				FileReference configFileLocation = new(testFilePath);

				Assert.IsTrue(FileReference.Exists(configFileLocation), "Unable to acquire a test file.");
				
				configFile = new ConfigFile(configFileLocation);

				FileReference.MakeWriteable(configFileLocation);
				configFile.Write(configFileLocation);

				Assert.IsNotNull(temporaryConfigFile.TemporaryFile);
				string fileContents = File.ReadAllText(temporaryConfigFile.TemporaryFile);
				Assert.AreEqual(testContents.Trim(), fileContents.Trim(), "The file contents after writing do not match the expected contents.");
			}
		}
	}
}
