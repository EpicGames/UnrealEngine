// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildTool.Tests.TestUtilities;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class CppCompileWarningsTests
	{
		internal class TestBuildSettingsProvider : IBuildContextProvider
		{
			internal BuildSettingsVersion _testBuildSettings = BuildSettingsVersion.V1;

			public BuildSettingsVersion GetBuildSettings()
			{
				return _testBuildSettings;
			}
		}

		[TestMethod]
		[DataRow(WarningLevel.Default, null, true, WarningLevel.Default)]                   // Null applied, no overwrite
		[DataRow(WarningLevel.Default, WarningLevel.Off, true, WarningLevel.Off)]           // Overwrite with Off
		[DataRow(WarningLevel.Default, WarningLevel.Warning, true, WarningLevel.Warning)]   // Overwrite with Warning
		[DataRow(WarningLevel.Default, WarningLevel.Error, true, WarningLevel.Error)]       // Overwrite with Error
		[DataRow(WarningLevel.Default, WarningLevel.Default, true, WarningLevel.Default)]   // Applied is Default, no change

		[DataRow(WarningLevel.Default, WarningLevel.Warning, false, WarningLevel.Warning)]  // Default, non-overwrite, change
		[DataRow(WarningLevel.Default, WarningLevel.Error, false, WarningLevel.Error)]      // Default, non-overwrite, change

		[DataRow(WarningLevel.Off, null, true, WarningLevel.Off)]                           // Null applied, no change
		[DataRow(WarningLevel.Off, WarningLevel.Default, true, WarningLevel.Off)]           // Applied Default, no change
		[DataRow(WarningLevel.Off, WarningLevel.Warning, true, WarningLevel.Warning)]       // Overwrite with Warning
		[DataRow(WarningLevel.Off, WarningLevel.Error, true, WarningLevel.Error)]           // Overwrite with Error
		[DataRow(WarningLevel.Off, WarningLevel.Warning, false, WarningLevel.Off)]          // No overwrite, no change

		[DataRow(WarningLevel.Warning, WarningLevel.Error, true, WarningLevel.Error)]       // Overwrite with Error
		[DataRow(WarningLevel.Warning, WarningLevel.Off, true, WarningLevel.Off)]           // Overwrite with Off
		[DataRow(WarningLevel.Warning, WarningLevel.Off, false, WarningLevel.Warning)]      // No overwrite, remains Warning

		[DataRow(WarningLevel.Error, WarningLevel.Off, true, WarningLevel.Off)]             // Overwrite with Off
		[DataRow(WarningLevel.Error, WarningLevel.Warning, true, WarningLevel.Warning)]     // Overwrite with Warning
		[DataRow(WarningLevel.Error, WarningLevel.Off, false, WarningLevel.Error)]          // No overwrite, remains Error
		public void TestApplyWarnings(WarningLevel inSourceWarningLevel, WarningLevel? appliedWarningLevel, bool overwriteSourceValue, WarningLevel expected)
		{
			WarningLevel sourceWarningLevel = inSourceWarningLevel;
			sourceWarningLevel.ApplyWarning(appliedWarningLevel, overwriteSourceValue);
			Assert.AreEqual(expected, sourceWarningLevel);
		}

		[TestMethod]
		public void TestDefaultBuildSettings()
		{
			CppCompileWarnings defaultBuildSettings = new();

			Assert.AreEqual(WarningLevel.Warning, defaultBuildSettings.ShadowVariableWarningLevel);
			Assert.AreEqual(WarningLevel.Off, defaultBuildSettings.UndefinedIdentifierWarningLevel);
			Assert.AreEqual(WarningLevel.Off, defaultBuildSettings.UnreachableCodeWarningLevel);
			Assert.AreEqual(WarningLevel.Off, defaultBuildSettings.UnsafeTypeCastWarningLevel);
			Assert.AreEqual(WarningLevel.Off, defaultBuildSettings.SwitchUnhandledEnumeratorWarningLevel);
		}

		[TestMethod]
		public void TestClonedMutableReturn()
		{
			CppCompileWarnings defaultBuildSettings = new();
			defaultBuildSettings.ShadowVariableWarningLevel = WarningLevel.Error;
			defaultBuildSettings.UndefinedIdentifierWarningLevel = WarningLevel.Warning;
			defaultBuildSettings.UnreachableCodeWarningLevel = WarningLevel.Warning;
			defaultBuildSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			defaultBuildSettings.SwitchUnhandledEnumeratorWarningLevel = WarningLevel.Warning;

			ReadOnlyCppCompileWarnings wrapper = new(defaultBuildSettings);

			CppCompileWarnings cloned = wrapper.CloneAsWriteable();

			{
				Assert.AreEqual(defaultBuildSettings.ShadowVariableWarningLevel, cloned.ShadowVariableWarningLevel);
				Assert.AreEqual(defaultBuildSettings.UndefinedIdentifierWarningLevel, cloned.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(defaultBuildSettings.UnreachableCodeWarningLevel, cloned.UnreachableCodeWarningLevel);
				Assert.AreEqual(defaultBuildSettings.UnsafeTypeCastWarningLevel, cloned.UnsafeTypeCastWarningLevel);
				Assert.AreEqual(defaultBuildSettings.SwitchUnhandledEnumeratorWarningLevel, cloned.SwitchUnhandledEnumeratorWarningLevel);
			}

			// Modify and assert != to each other - the same underyling reference has not been returned
			{
				cloned.ShadowVariableWarningLevel = WarningLevel.Warning;
				cloned.UndefinedIdentifierWarningLevel = WarningLevel.Error;
				cloned.UnreachableCodeWarningLevel = WarningLevel.Error;
				cloned.UnsafeTypeCastWarningLevel = WarningLevel.Warning;
				cloned.SwitchUnhandledEnumeratorWarningLevel = WarningLevel.Error;

				Assert.AreNotEqual(defaultBuildSettings.ShadowVariableWarningLevel, cloned.ShadowVariableWarningLevel);
				Assert.AreNotEqual(defaultBuildSettings.UndefinedIdentifierWarningLevel, cloned.UndefinedIdentifierWarningLevel);
				Assert.AreNotEqual(defaultBuildSettings.UnreachableCodeWarningLevel, cloned.UnreachableCodeWarningLevel);
				Assert.AreNotEqual(defaultBuildSettings.UnsafeTypeCastWarningLevel, cloned.UnsafeTypeCastWarningLevel);
				Assert.AreNotEqual(defaultBuildSettings.SwitchUnhandledEnumeratorWarningLevel, cloned.SwitchUnhandledEnumeratorWarningLevel);
			}
		}

		[TestMethod]
		public void TestContextBuildSettingsOverride()
		{
			TestBuildSettingsProvider buildContextProvider = new();
			buildContextProvider._testBuildSettings = BuildSettingsVersion.V1;

			CppCompileWarnings newWarnings = new(buildContextProvider, null);

			// Shadow Variable build setting defaults
			{
				Assert.AreEqual(WarningLevel.Default, newWarnings.ShadowVariableWarningLevel);

				// Boundary analysis
				buildContextProvider._testBuildSettings = BuildSettingsVersion.V2;
				CppCompileWarnings.ApplyTargetDefaults(newWarnings, true);
				Assert.AreEqual(WarningLevel.Error, newWarnings.ShadowVariableWarningLevel);

				buildContextProvider._testBuildSettings = BuildSettingsVersion.V3;

				CppCompileWarnings.ApplyTargetDefaults(newWarnings, true);
				Assert.AreEqual(WarningLevel.Error, newWarnings.ShadowVariableWarningLevel);
			}
		}

		[TestMethod]
		public void TestParentSettings()
		{
			TestBuildSettingsProvider buildContextProvider = new();
			buildContextProvider._testBuildSettings = BuildSettingsVersion.V1;
			CppCompileWarnings parentWarnings = new(buildContextProvider, null);
			CppCompileWarnings newWarnings = new(buildContextProvider, null, new ReadOnlyCppCompileWarnings(parentWarnings));

			// Assert invariants
			{
				Assert.AreEqual(WarningLevel.Default, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UnsafeTypeCastWarningLevel);
			}

			// Assert new parent values
			{
				parentWarnings.ShadowVariableWarningLevel = WarningLevel.Error;
				parentWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Error;
				parentWarnings.UnreachableCodeWarningLevel = WarningLevel.Error;
				parentWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

				Assert.AreEqual(WarningLevel.Error, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnsafeTypeCastWarningLevel);
			}
		}

		[TestMethod]
		public void TestReparentOperations()
		{
			TestBuildSettingsProvider buildContextProvider = new();
			buildContextProvider._testBuildSettings = BuildSettingsVersion.V1;
			CppCompileWarnings parentWarnings = new(buildContextProvider, null);
			CppCompileWarnings newWarnings = new(buildContextProvider, null, new ReadOnlyCppCompileWarnings(parentWarnings));

			// Assert invariant parent values
			{
				parentWarnings.ShadowVariableWarningLevel = WarningLevel.Error;
				parentWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Error;
				parentWarnings.UnreachableCodeWarningLevel = WarningLevel.Error;
				parentWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

				Assert.AreEqual(WarningLevel.Error, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnsafeTypeCastWarningLevel);
			}

			// Assert reparenting
			CppCompileWarnings parentWarnings2 = new(buildContextProvider, null);
			{
				newWarnings.AddParent(parentWarnings2);
				parentWarnings2.ShadowVariableWarningLevel = WarningLevel.Warning;
				parentWarnings2.UndefinedIdentifierWarningLevel = WarningLevel.Warning;
				parentWarnings2.UnreachableCodeWarningLevel = WarningLevel.Warning;
				parentWarnings2.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

				Assert.AreEqual(WarningLevel.Warning, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UnsafeTypeCastWarningLevel);
			}

			// Assert transitive to eldest ancestor
			{
				parentWarnings2.ShadowVariableWarningLevel = WarningLevel.Default;
				Assert.AreEqual(parentWarnings.ShadowVariableWarningLevel, newWarnings.ShadowVariableWarningLevel);
			}

			// Assert removal of parentWarnings2 - UndefinedIdentifierWarningLevel, UnreachableCodeWarningLevel, and UnsafeTypeCastWarningLevel will now be the resolved items
			{
				newWarnings.RemoveParent();
				Assert.AreEqual(parentWarnings.UndefinedIdentifierWarningLevel, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(parentWarnings.UnreachableCodeWarningLevel, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(parentWarnings.UnsafeTypeCastWarningLevel, newWarnings.UnsafeTypeCastWarningLevel);

			}

			// Removal of only parent; should only be default values
			{
				newWarnings.RemoveParent();
				Assert.AreEqual(WarningLevel.Default, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Default, newWarnings.UnsafeTypeCastWarningLevel);
			}
		}

		[TestMethod]
		public void TestChildOverrides()
		{
			TestBuildSettingsProvider buildContextProvider = new();
			buildContextProvider._testBuildSettings = BuildSettingsVersion.V1;
			CppCompileWarnings parentWarnings = new(buildContextProvider, null);
			CppCompileWarnings newWarnings = new(buildContextProvider, null, new ReadOnlyCppCompileWarnings(parentWarnings));

			parentWarnings.ShadowVariableWarningLevel = WarningLevel.Error;
			parentWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Error;
			parentWarnings.UnreachableCodeWarningLevel = WarningLevel.Error;
			parentWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			// Assert invariants
			{
				Assert.AreEqual(WarningLevel.Error, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Error, newWarnings.UnsafeTypeCastWarningLevel);
			}

			// Child overrides;
			newWarnings.ShadowVariableWarningLevel = WarningLevel.Warning;
			newWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Warning;
			newWarnings.UnreachableCodeWarningLevel = WarningLevel.Warning;
			newWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			// Assert new parent values
			{
				Assert.AreEqual(WarningLevel.Warning, newWarnings.ShadowVariableWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UndefinedIdentifierWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UnreachableCodeWarningLevel);
				Assert.AreEqual(WarningLevel.Warning, newWarnings.UnsafeTypeCastWarningLevel);
			}
		}

		[TestMethod]
		public void TestGenerateWarningCommandLineAgsOrder()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;
			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);

				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);
				IEnumerable<string> args = null!;

				RulesAssembly generatedTestGameRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStub", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRulesNonWin64 = generatedTestGameRulesAssembly.CreateTargetRules("TestStub", UnrealTargetPlatform.Android, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				CppCompileWarnings specializedWarnings = new(testTargetRulesNonWin64, null);

				// Verify the ordering
				{
					VersionNumber newVersionNumber = new(18, 9, 9);
					specializedWarnings.DeprecatedCopyWarningLevel = WarningLevel.Off;
					specializedWarnings.DeprecatedCopyWithUserProvidedCopyWarningLevel = WarningLevel.Error;
					args = specializedWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

					Assert.IsTrue(args.Count() == 2);
					Assert.IsTrue(args.First() == "-Wno-deprecated-copy");
					Assert.IsTrue(args.Last() == "-Wdeprecated-copy-with-user-provided-copy");
				}
			}
		}

		[TestMethod]
		public void TestGenerateWarningCommandLineAgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);

				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);

				// Generate rules assembly & subsequent targets
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				IEnumerable<string> args = null!;

				// No args, base check
				{
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));
					Assert.IsFalse(args.Any());
				}

				// Specialized VC toolchain check; UndefinedIdentifier
				{
					baseWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Warning;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("/w44668"));

					CppCompileEnvironment preprocessOnlyCompileEnvironment = new(newCompileEnvironment);
					preprocessOnlyCompileEnvironment.bPreprocessOnly = true;
					args = baseWarnings.GenerateWarningCommandLineArgs(preprocessOnlyCompileEnvironment, typeof(VCToolChain));

					Assert.IsFalse(args.Any());

					baseWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Default;
				}

				// Specialized VC toolchain check; UnreachableCode
				{
					baseWarnings.UnreachableCodeWarningLevel = WarningLevel.Warning;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("/w44702"));

					CppCompileEnvironment preprocessOnlyCompileEnvironment = new(newCompileEnvironment);
					preprocessOnlyCompileEnvironment.bPreprocessOnly = true;
					args = baseWarnings.GenerateWarningCommandLineArgs(preprocessOnlyCompileEnvironment, typeof(VCToolChain));

					Assert.IsFalse(args.Any());

					baseWarnings.UnreachableCodeWarningLevel = WarningLevel.Default;
				}

				{
					// DeterministicWarningLevel
					VersionNumber newVersionNumber = new(18, 9, 9);

					CppCompileEnvironment deterministicCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
					deterministicCompileEnvironment.bDeterministic = false;

					args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(VCToolChain), newVersionNumber);
					Assert.IsFalse(args.Any());

					deterministicCompileEnvironment.bDeterministic = true;

					args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(VCToolChain), newVersionNumber);
					Assert.IsFalse(args.Any());

					baseWarnings.DeterministicWarningLevel = WarningLevel.Off;
					args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(VCToolChain), newVersionNumber);

					Assert.AreEqual(1, args.Count());
					Assert.IsTrue(args.Contains("/wd5048"));

					baseWarnings.DeterministicWarningLevel = WarningLevel.Warning;
					args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(VCToolChain), newVersionNumber);

					Assert.AreEqual(0, args.Count());
					Assert.IsFalse(args.Any());

					baseWarnings.DeterministicWarningLevel = WarningLevel.Error;
					args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(VCToolChain), newVersionNumber);

					Assert.AreEqual(1, args.Count());
					Assert.IsTrue(args.Contains("/we5048"));
				}
			}
		}

		[TestMethod]
		public void TestSpecializedClangCommandLineArgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);

				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);

				// Generate rules assembly & subsequent targets
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				IEnumerable<string> args = null!;
				{
					// DeterministcFlagSetFilter
					{
						testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
						baseWarnings.DeterministicWarningLevel = WarningLevel.Off;
						VersionNumber newVersionNumber = new(18, 9, 9);

						CppCompileEnvironment deterministicCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
						deterministicCompileEnvironment.bDeterministic = false;

						args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
						Assert.IsFalse(args.Any());

						deterministicCompileEnvironment.bDeterministic = true;

						args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
						Assert.IsTrue(args.Count() == 1);
						Assert.IsTrue(args.Contains("-Wno-date-time"));

						deterministicCompileEnvironment.bDeterministic = false;
						baseWarnings.DeterministicWarningLevel = WarningLevel.Warning;
						args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
						Assert.IsFalse(args.Any());

						deterministicCompileEnvironment.bDeterministic = true;
						args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

						Assert.IsTrue(args.Count() == 2);
						Assert.IsTrue(args.Contains("-Wdate-time"));
						Assert.IsTrue(args.Contains("-Wno-error=date-time"));

						baseWarnings.DeterministicWarningLevel = WarningLevel.Error;
						args = baseWarnings.GenerateWarningCommandLineArgs(deterministicCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
						Assert.IsTrue(args.Count() == 1);
						Assert.IsTrue(args.Contains("-Wdate-time"));

						baseWarnings.DeterministicWarningLevel = WarningLevel.Default;
					}

					// PGOOptimizedFilter
					{
						testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
						baseWarnings.ProfileInstructWarningLevel = WarningLevel.Off;
						VersionNumber newVersionNumber = new(18, 9, 9);

						CppCompileEnvironment pgoOptimizedCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
						pgoOptimizedCompileEnvironment.bPGOOptimize = false;

						args = baseWarnings.GenerateWarningCommandLineArgs(pgoOptimizedCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
						Assert.IsFalse(args.Any());

						pgoOptimizedCompileEnvironment.bPGOOptimize = true;

						args = baseWarnings.GenerateWarningCommandLineArgs(pgoOptimizedCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

						Assert.IsTrue(args.Count() == 2);
						Assert.IsTrue(args.Contains("-Wno-profile-instr-out-of-date"));
						Assert.IsTrue(args.Contains("-Wno-profile-instr-unprofiled"));

						baseWarnings.ProfileInstructWarningLevel = WarningLevel.Off;
					}

					// AndroidNDKR28ToolChainVersionExclusion
					{
						testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
						baseWarnings.CastFunctionTypeMismatchWarningLevel = WarningLevel.Off;

						VersionNumber newVersionNumber = new(19, 0, 0);
						CppCompileEnvironment androidCompileEnvironment = new(UnrealTargetPlatform.Android, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);
						args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

						Assert.IsFalse(args.Any());

						args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

						Assert.IsTrue(args.Count() == 1);
						Assert.IsTrue(args.Contains("-Wno-cast-function-type-mismatch"));

						newVersionNumber = new VersionNumber(19, 0, 1);
						args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

						Assert.IsTrue(args.Count() == 1);
						Assert.IsTrue(args.Contains("-Wno-cast-function-type-mismatch"));

						baseWarnings.CastFunctionTypeMismatchWarningLevel = WarningLevel.Default;
					}

					// AndroidNDKR26ToolChainVersionExclusion
					{
						testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
						baseWarnings.InvalidUnevaluatedStringWarningLevel = WarningLevel.Off;
						CppCompileEnvironment androidCompileEnvironment = new(UnrealTargetPlatform.Android, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);

						// Exclusion cases
						{
							VersionNumber newVersionNumber = new(17, 0, 2);
							args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

							Assert.IsTrue(args.Count() == 1 && args.Contains("-Wno-shadow"));

							newVersionNumber = new VersionNumber(16);
							args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

							Assert.IsFalse(args.Any());

							newVersionNumber = new VersionNumber(17, 1);
							androidCompileEnvironment.CppStandard = CppStandardVersion.Latest;
							args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

							Assert.IsTrue(args.Count() == 1 && args.Contains("-Wno-shadow"));
						}

						// Inclusion cases
						{
							VersionNumber newVersionNumber = new(17, 1);
							androidCompileEnvironment.CppStandard = CppStandardVersion.Cpp20;

							args = baseWarnings.GenerateWarningCommandLineArgs(androidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

							Assert.IsTrue(args.Contains("-Wno-invalid-unevaluated-string"));

							newVersionNumber = new VersionNumber(17, 0, 2);

							args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);

							Assert.IsTrue(args.Contains("-Wno-invalid-unevaluated-string"));
						}

						baseWarnings.InvalidUnevaluatedStringWarningLevel = WarningLevel.Default;
					}

					// UnusedValueClangToolChainAttribute
					{
						testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
						baseWarnings.UnusedValueWarningLevel = WarningLevel.Off;
						VersionNumber newVersionNumber = new(18, 9, 9);

						CppCompileEnvironment invalidCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
						CppCompileEnvironment validCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Shipping, new UnrealArchitectures(UnrealArch.X64), null!);

						args = baseWarnings.GenerateWarningCommandLineArgs(invalidCompileEnvironment, typeof(ClangToolChain), newVersionNumber, StaticAnalyzer.None);
						Assert.IsFalse(args.Any());

						args = baseWarnings.GenerateWarningCommandLineArgs(validCompileEnvironment, typeof(ClangToolChain), newVersionNumber, StaticAnalyzer.None);
						Assert.IsTrue(args.Count() == 2 && args.Contains("-Wno-unused-value"));

						args = baseWarnings.GenerateWarningCommandLineArgs(validCompileEnvironment, typeof(ClangToolChain), newVersionNumber, StaticAnalyzer.Clang);
						Assert.IsTrue(args.Count() == 2 && args.Contains("-Wno-unused-value"));
					}
				}
			}
		}

		[TestMethod]
		public void TestConfigurationFilterCommandLineArgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);
				IEnumerable<string> args = null!;

				// ConfigurationFilterAttribute
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.UnusedValueWarningLevel = WarningLevel.Off;
					VersionNumber newVersionNumber = new(18, 9, 9);

					CppCompileEnvironment invalidCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
					CppCompileEnvironment validCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Shipping, new UnrealArchitectures(UnrealArch.X64), null!);

					args = baseWarnings.GenerateWarningCommandLineArgs(invalidCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsFalse(args.Any());

					args = baseWarnings.GenerateWarningCommandLineArgs(validCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 2);
					Assert.IsTrue(args.Contains("-Wno-unused-value"));

					baseWarnings.UnusedValueWarningLevel = WarningLevel.Default;
				}
			}
		}

		[TestMethod]
		public void TestCppStandardConstraintCommandLineArgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);
				IEnumerable<string> args = null!;

				// CppStandardConstrainedFilterAttribute
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.AmbiguousReversedOperatorWarningLevel = WarningLevel.Off;
					VersionNumber newVersionNumber = new(18, 9, 9);

					CppCompileEnvironment cppStandardCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Debug, new UnrealArchitectures(UnrealArch.X64), null!);
#pragma warning disable CS0618 // Type or member is obsolete
					cppStandardCompileEnvironment.CppStandard = CppStandardVersion.Cpp17;
#pragma warning restore CS0618 // Type or member is obsolete

					args = baseWarnings.GenerateWarningCommandLineArgs(cppStandardCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsFalse(args.Any());

					cppStandardCompileEnvironment.CppStandard = CppStandardVersion.Cpp20;
					args = baseWarnings.GenerateWarningCommandLineArgs(cppStandardCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-ambiguous-reversed-operator"));

					cppStandardCompileEnvironment.CppStandard = CppStandardVersion.Latest;
					args = baseWarnings.GenerateWarningCommandLineArgs(cppStandardCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-ambiguous-reversed-operator"));

					baseWarnings.AmbiguousReversedOperatorWarningLevel = WarningLevel.Default;
				}
			}
		}

		[TestMethod]
		public void TestVersionConstraintCommandLineArgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);
				IEnumerable<string> args = null!;

				// Specialized VC toolchain check;  UnsafeTypeCast
				{
					RulesAssembly generatedTestGameRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStub", false, false, false, null, false, new TestLogger());
					TargetRules testTargetRulesNonWin64 = generatedTestGameRulesAssembly.CreateTargetRules("TestStub", UnrealTargetPlatform.Android, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
					CppCompileWarnings specializedWarnings = new(testTargetRulesNonWin64, null);

					specializedWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;
					args = specializedWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsTrue(args.Count() == 2);
					Assert.IsTrue(args.Contains("/wd4244"));
					Assert.IsTrue(args.Contains("/wd4838"));

					baseWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsTrue(args.Count() == 2);
					Assert.IsTrue(args.Contains("/w44244"));
					Assert.IsTrue(args.Contains("/w44838"));

					baseWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Default;
				}

				// Clang toolchain version check - no version; should disable Shadow
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Error;

					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain));
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-shadow"));

					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Default;
				}

				// Clang toolchain version check - valid versions;
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Error;

					VersionNumber newVersionNumber = new(16, 9, 5);

					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wshadow"));

					newVersionNumber = new VersionNumber(17);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-shadow"));

					newVersionNumber = new VersionNumber(17, 1);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-shadow"));

					newVersionNumber = new VersionNumber(18, 1, 2);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-shadow"));

					newVersionNumber = new VersionNumber(18, 1, 3);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wshadow"));

					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Default;
				}

				// ToolChainVersionConstrainedFilterAttribute
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.Shorten64To32WarningLevel = WarningLevel.Off;

					VersionNumber newVersionNumber = new(18, 9, 9);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsFalse(args.Any());

					//
					newVersionNumber = new VersionNumber(19, 0, 0);
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(ClangToolChain), newVersionNumber);
					Assert.IsTrue(args.Count() == 1);
					Assert.IsTrue(args.Contains("-Wno-shorten-64-to-32"));
					baseWarnings.Shorten64To32WarningLevel = WarningLevel.Default;
				}
			}
		}

		[TestMethod]
		public void TestMSVCAndVCClangGenerateWarningCommandLineArgs()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStub.Target.ubttest")), "TestStub.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				CppCompileWarnings baseWarnings = new(testTargetRules, null);

				CppCompileEnvironment newCompileEnvironment = new(UnrealTargetPlatform.Win64, CppConfiguration.Development, new UnrealArchitectures(UnrealArch.X64), null!);
				IEnumerable<string> args = null!;

				// VC toolchain check;
				{
					baseWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));
					Assert.IsTrue(args.Count() == 2);
					Assert.IsTrue(args.Contains("/w44244"));
					Assert.IsTrue(args.Contains("/w44838"));

					baseWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Default;
				}

				// MSVC compiler check;
				{
					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Error;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsTrue(args.Count() == 3);
					Assert.IsTrue(args.Contains("/we4456"));
					Assert.IsTrue(args.Contains("/we4458"));
					Assert.IsTrue(args.Contains("/we4459"));

					WindowsCompiler previousCompiler = testTargetRules.WindowsPlatform.Compiler;
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Intel;

					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));

					Assert.IsFalse(args.Any());

					testTargetRules.WindowsPlatform.Compiler = previousCompiler;
					baseWarnings.ShadowVariableWarningLevel = WarningLevel.Default;
				}

				// VCClangCompilerFilter
				{
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.Clang;
					baseWarnings.MicrosoftGroupWarningLevel = WarningLevel.Off;

					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));
					Assert.IsTrue(args.Any());
					Assert.IsTrue(args.Contains("-Wno-microsoft"));

					// make sure exclusively ONLY the clang stuff is in:
					testTargetRules.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
					args = baseWarnings.GenerateWarningCommandLineArgs(newCompileEnvironment, typeof(VCToolChain));
					Assert.IsFalse(args.Any());
				}
			}
		}
	}
}