// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using JsonExtensions;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildBase;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class JsonExtensionsTests
	{
		public enum TestEnum
		{
			A,
			B,
			C,
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackReturnsNullWhenBothAbsent()
		{
			JsonObject json = new();
			TestEnum[]? values = json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old");
			Assert.IsNull(values);
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackThrowsWhenBothPresent()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("New", [TestEnum.A, TestEnum.B]);
			json.AddOrSetFieldValue("Old", [TestEnum.A, TestEnum.B]);
			BuildLogEventException ex = Assert.ThrowsExactly<BuildLogEventException>(
				() => json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old"));
			StringAssert.Contains(ex.Message, "New", StringComparison.Ordinal);
			StringAssert.Contains(ex.Message, "Old", StringComparison.Ordinal);
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackPicksUpNewValues()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("New", [TestEnum.A, TestEnum.B]);
			TestEnum[]? values = json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old");
			CollectionAssert.AreEquivalent(values, new[] { TestEnum.A, TestEnum.B });
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackPicksUpFallbackValues()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("Old", [TestEnum.A, TestEnum.B]);
			TestEnum[]? values = json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old");
			CollectionAssert.AreEquivalent(values, new[] { TestEnum.A, TestEnum.B });
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackFailsOnInvalidNewValue()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("New", ["A", "B", "C", "BadItem1", "BadItem2"]);
			BuildLogEventException ex = Assert.ThrowsExactly<BuildLogEventException>(
				() => json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old"));
			StringAssert.Contains(ex.Message, "New", StringComparison.Ordinal);
			StringAssert.Contains(ex.Message, "BadItem1", StringComparison.Ordinal);
			StringAssert.Contains(ex.Message, "BadItem2", StringComparison.Ordinal);
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackFailsOnInvalidFallbackValue()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("Old", ["A", "B", "C", "BadItem1", "BadItem2"]);
			BuildLogEventException ex = Assert.ThrowsExactly<BuildLogEventException>(
				() => json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old"));
			StringAssert.Contains(ex.Message, "Old", StringComparison.Ordinal);
			StringAssert.Contains(ex.Message, "BadItem1", StringComparison.Ordinal);
			StringAssert.Contains(ex.Message, "BadItem2", StringComparison.Ordinal);
		}

		[TestMethod]
		public void TestGetEnumArrayFieldWithDeprecatedFallbackFailsOnInvalidDataType()
		{
			JsonObject json = new();
			json.AddOrSetFieldValue("New", "A");
			BuildLogEventException ex = Assert.ThrowsExactly<BuildLogEventException>(
				() => json.GetEnumArrayFieldWithDeprecatedFallback<TestEnum>("New", "Old"));
			StringAssert.Contains(ex.Message, "New", StringComparison.Ordinal);
		}
	}
}
