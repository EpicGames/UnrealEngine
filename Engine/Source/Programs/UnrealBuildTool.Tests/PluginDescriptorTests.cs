// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class PluginDescriptorTests
	{
		private static DirectoryReference? s_tempDirectory;

		private static DirectoryReference CreateTempDir()
		{
			string tempDir = Path.Join(Path.GetTempPath(), "epicgames-core-tests-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);
			return new DirectoryReference(tempDir);
		}

		[ClassInitialize]
		public static void Setup(TestContext _)
		{
			PluginDescriptorTests.s_tempDirectory = CreateTempDir();
		}

		[ClassCleanup]
		public static void TearDown()
		{
			if (s_tempDirectory != null && Directory.Exists(PluginDescriptorTests.s_tempDirectory.FullName))
			{
				Directory.Delete(PluginDescriptorTests.s_tempDirectory.FullName, true);
			}
		}

		[TestMethod]
		public void ReadPluginCustomFields()
		{
			string json = CreateUPluginWithCustomFields();
			string filename = "CustomFields.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			PluginDescriptor.FromFile(fileToWrite);
		}

		[TestMethod]
		public void ReadUpluginMissingFileVersion()
		{
			string json = CreateUpluginWithMissingVersion();
			string filename = "MissingFileVersionField.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			Assert.ThrowsExactly<BuildException>(() => PluginDescriptor.FromFile(fileToWrite));
		}

		[TestMethod]
		public void ReadUpluginInvalidVersion()
		{
			string json = CreateUpluginWithInvalidVersion();
			string filename = "InvalidFileVersionField.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			Assert.ThrowsExactly<BuildException>(() => PluginDescriptor.FromFile(fileToWrite));
		}

		[TestMethod]
		public void Save2CustomFieldsMissingDefaults()
		{
			string json = CreateUPluginWithCustomFields();
			string filename = "CustomFields-Save.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			PluginDescriptor descriptor = PluginDescriptor.FromFile(fileToWrite);
			string fileToSave = GetAbsolutePathToTempFile("CustomFields-SaveResults.uplugin");
			descriptor.Save2(fileToSave);

			string saveContents = File.ReadAllText(fileToSave);
			JsonObject writtenObject = JsonObject.Parse(json);
			JsonObject savedObject = JsonObject.Parse(saveContents);
			IEnumerable<string> writtenObjectKeys = writtenObject.KeyNames;
			// At this point, the written object should contain all the custom fields 
			// The saved object should have all the custom fields as well as the missing default fields appended
			// This means we can't do a simple equality check to see if the objects are the same 
			// We check to make sure that all of the keys in the written object exist in the saved object as a short hand to check that all the custom fields have been saved and carried over to the saved object 
			foreach (string key in writtenObjectKeys)
			{
				Assert.IsTrue(savedObject.ContainsField(key));
			}
		}

		[TestMethod]
		public void Save2DefaultFields()
		{
			string json = CreateUpluginWithDefaultFields();
			string filename = "DefaultFields-Save.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			PluginDescriptor descriptor = PluginDescriptor.FromFile(fileToWrite);
			string fileToSave = GetAbsolutePathToTempFile("DefaultFields-SaveResults.uplugin");
			descriptor.Save2(fileToSave);
			string saveContents = File.ReadAllText(fileToSave);

			// Nothing should change between the format of the json and what was written to disk 
			Assert.AreEqual(json, saveContents);
		}

		[TestMethod]
		public void Save2PluginGameFeatureActivation()
		{
			string json = CreateUpluginWithGameFeatureActivation();
			string filename = "GameFeatureActivation-Save.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);
			PluginDescriptor descriptor = PluginDescriptor.FromFile(fileToWrite);
			string fileToSave = GetAbsolutePathToTempFile("GameFeatureActivation-Save.uplugin");
			descriptor.Save2(fileToSave);
			string saveContents = File.ReadAllText(fileToSave);

			// Nothing should change between the format of the json and what was written to disk 
			Assert.AreEqual(json, saveContents);
		}

		[TestMethod]
		public void Save2CompareWithSave()
		{
			string json = CreateUpluginWithDefaultFields();
			string filename = "DefaultFields-Save2.uplugin";
			FileReference fileToWrite = CreateTempUPluginFile(filename, json);

			PluginDescriptor descriptor = PluginDescriptor.FromFile(fileToWrite);
			string fileToSave = GetAbsolutePathToTempFile("DefaultFields-SaveResults.uplugin");
			string fileToSave2 = GetAbsolutePathToTempFile("DefaultFields-Save2Results.uplugin");
			descriptor.Save2(fileToSave);
			descriptor.Save2(fileToSave2);

			string saveContents = File.ReadAllText(fileToSave);
			string save2Contents = File.ReadAllText(fileToSave2);

			// Nothing should change between the format of the json and what was written to disk 
			Assert.AreEqual(saveContents, save2Contents);
		}

		private static string GetAbsolutePathToTempFile(string fileName)
		{
			return String.IsNullOrEmpty(fileName) || s_tempDirectory == null ? "" : Path.Join(s_tempDirectory.FullName, fileName);
		}

		private static FileReference CreateTempUPluginFile(string fileName, string fileContent)
		{
			string extension = Path.GetExtension(fileName);
			Debug.Assert(!String.IsNullOrEmpty(extension));
			Debug.Assert(extension == ".uplugin");
			string inputFile = Path.Join(s_tempDirectory?.FullName, fileName);
			FileReference inputFileReference = new(inputFile);
			File.WriteAllText(inputFile, fileContent);
			return inputFileReference;
		}

		private static string CreateUPluginWithCustomFields()
		{
			string json = @"
				{
					""FileVersion"": 3,
					""CanContainContent"": true,
					""ExplicitlyLoaded"": true,
					""EnabledByDefault"": false,
					""EditorCustomVirtualPath"": ""MyPlugin"",
					""CustomField1"": ""400.0"",
					""CustomField2"": ""CustomValue2"",
					""CustomField3"": false,
					""Modules"": [
						{
							""Name"": ""Module1"",
							""Type"": ""Runtime"",
							""LoadingPhase"": ""Default""
						},
						{
							""Name"": ""Module2"",
							""Type"": ""ClientOnly"",
							""LoadingPhase"": ""Default""
						}
					],
					""Plugins"": [
						{
							""Name"": ""Plugin1"",
							""Enabled"": true
						},
						{
							""Name"": ""Plugin2"",
							""Enabled"": true
						},
						{
							""Name"": ""Plugin3"",
							""Enabled"": true
						},
						{
							""Name"": ""Plugin4"",
							""Enabled"": true
						},
						{
							""Name"": ""Plugin5"",
							""Enabled"": true
						}
					]
				}";

			return FixVerbatimStringIdentation(json);
		}

		private static string CreateUpluginWithDefaultFields()
		{
			string json = @"
				{
					""FileVersion"": 3,
					""Version"": 1,
					""VersionName"": ""1.0"",
					""FriendlyName"": ""Enhanced Input"",
					""Description"": ""Input handling that allows for contextual and dynamic mappings."",
					""Category"": ""Input"",
					""CreatedBy"": ""Epic Games, Inc."",
					""CreatedByURL"": ""https://epicgames.com"",
					""DocsURL"": ""https://docs.unrealengine.com/en-US/enhanced-input-in-unreal-engine/"",
					""MarketplaceURL"": """",
					""SupportURL"": """",
					""EnabledByDefault"": true,
					""CanContainContent"": false,
					""IsBetaVersion"": false,
					""Installed"": false,
					""Modules"": [
						{
							""Name"": ""EnhancedInput"",
							""Type"": ""Runtime"",
							""LoadingPhase"": ""PreDefault""
						},
						{
							""Name"": ""InputBlueprintNodes"",
							""Type"": ""UncookedOnly"",
							""LoadingPhase"": ""Default""
						},
						{
							""Name"": ""InputEditor"",
							""Type"": ""Editor"",
							""LoadingPhase"": ""Default""
						}
					]
				}";

			return FixVerbatimStringIdentation(json);
		}

		private static string CreateUpluginWithMissingVersion()
		{
			string json = @"
				{ 
					""CustomField1"": ""Value1""
				}";
			return FixVerbatimStringIdentation(json);
		}

		private static string CreateUpluginWithInvalidVersion()
		{
			string json = @"
				{ 
					""FileVersion"": 4
				}";
			return FixVerbatimStringIdentation(json);
		}

		private static string CreateUpluginWithGameFeatureActivation()
		{
			string json = @"
				{
					""FileVersion"": 3,
					""Version"": 1,
					""VersionName"": """",
					""FriendlyName"": """",
					""Description"": """",
					""Category"": """",
					""CreatedBy"": """",
					""CreatedByURL"": """",
					""DocsURL"": """",
					""MarketplaceURL"": """",
					""SupportURL"": """",
					""CanContainContent"": true,
					""ExplicitlyLoaded"": true,
					""EnabledByDefault"": false,
					""EditorCustomVirtualPath"": ""MyPlugin"",
					""Modules"": [
						{
							""Name"": ""Module1"",
							""Type"": ""Runtime"",
							""LoadingPhase"": ""Default""
						},
						{
							""Name"": ""Module2"",
							""Type"": ""ClientOnly"",
							""LoadingPhase"": ""Default""
						}
					],
					""Plugins"": [
						{
							""Name"": ""Plugin1"",
							""Enabled"": true
						},
						{
							""Name"": ""Plugin2"",
							""Enabled"": true,
							""Activate"": true
						},
						{
							""Name"": ""Plugin3"",
							""Enabled"": true,
							""Optional"": true,
							""Activate"": true
						},
						{
							""Name"": ""Plugin4"",
							""Enabled"": true,
							""Optional"": true
						}
					]
				}";
			return FixVerbatimStringIdentation(json);
		}

		private static string FixVerbatimStringIdentation(string verbatimString)
		{
			if (String.IsNullOrEmpty(verbatimString))
			{
				return verbatimString;
			}
			string[] lines = verbatimString.Split([Environment.NewLine], StringSplitOptions.None);
			StringBuilder jsonStringBuilder = new();
			int numberOfTabs = 0;
			bool bFoundTabLevel = false;
			foreach (string line in lines)
			{
				// Should only apply to the first line. Skip 
				if (String.IsNullOrEmpty(line))
				{
					continue;
				}
				// Now we're on the first line with the { we're just going to assume that this is the first line 
				if (!bFoundTabLevel)
				{
					numberOfTabs = line.TakeWhile(x => x == '\t').Count();
					bFoundTabLevel = true;
				}
				jsonStringBuilder.AppendLine(line.Substring(numberOfTabs));
			}
			return jsonStringBuilder.ToString();
		}
	}
}
