// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class PreprocessorExpressionTests
	{
		[TestMethod]
		public void Addition()
		{
			RunTest("1 + 2 > 3", 0);
			RunTest("1 + 2 >= 3", 1);
		}

		[TestMethod]
		public void Equality()
		{
			RunTest("1 == 1", 1);
			RunTest("1 == 2", 0);
			RunTest("1 != 1", 0);
			RunTest("1 != 2", 1);
			RunTest("1 < 0", 0);
			RunTest("1 < 1", 0);
			RunTest("1 < 2", 1);
			RunTest("1 <= 0", 0);
			RunTest("1 <= 1", 1);
			RunTest("1 <= 2", 1);
			RunTest("1 > 0", 1);
			RunTest("1 > 1", 0);
			RunTest("1 > 2", 0);
			RunTest("1 >= 0", 1);
			RunTest("1 >= 1", 1);
			RunTest("1 >= 2", 0);
		}

		[TestMethod]
		public void UnaryOperators()
		{
			RunTest("0 + 1 == +1", 1);
			RunTest("0 - 1 == -1", 1);
			RunTest("!0", 1);
			RunTest("!1", 0);
			RunTest("~0", -1L);
		}

		[TestMethod]
		public void ArithmeticOperators()
		{
			RunTest("3 + 7", 10);
			RunTest("3 - 7", -4);
			RunTest("3 * 7", 21);
			RunTest("21 / 3", 7);
			RunTest("24 % 7", 3);
			RunTest("2 << 4", 32);
			RunTest("32 >> 4", 2);
			RunTest("(0xab & 0xf)", 0xb);
			RunTest("(0xab | 0xf)", 0xaf);
			RunTest("(0xab ^ 0xf)", 0xa4);
		}

		[TestMethod]
		public void LogicalOperators()
		{
			RunTest("0 || 0", 0);
			RunTest("0 || 1", 1);
			RunTest("1 || 0", 1);
			RunTest("1 || 1", 1);
			RunTest("0 && 0", 0);
			RunTest("0 && 1", 0);
			RunTest("1 && 0", 0);
			RunTest("1 && 1", 1);
		}

		[TestMethod]
		public void TernaryOperator()
		{
			RunTest("((0)? 123 : 456) == 456", 1);
			RunTest("((1)? 123 : 456) == 456", 0);
		}

		[TestMethod]
		public void Precedence()
		{
			RunTest("3 + 27 / 3", 12);
			RunTest("(3 + 27) / 3", 10);
		}

		[TestMethod]
		public void Literals()
		{
			RunTest("123", 123);
			RunTest("123l", 123);
			RunTest("123L", 123);
			RunTest("123ll", 123);
			RunTest("123LL", 123);
			RunTest("123u", 123);
			RunTest("123U", 123);
			RunTest("123ul", 123);
			RunTest("123Ul", 123);
			RunTest("123UL", 123);
			RunTest("123Ull", 123);
			RunTest("123ULL", 123);
			RunTest("123ui64", 123);
			RunTest("0x123i64", 0x123);
			RunTest("0123i64", 83);
		}

		class PreprocessorTestContext : PreprocessorContext
		{
			public PreprocessorTestContext()
				: base(null)
			{
			}
		}

		/// <summary>
		/// Tokenize the given expression and evaluate it, and check it matches the expected result
		/// </summary>
		/// <param name="expression">The expression to evaluate, as a string</param>
		/// <param name="expectedResult">The expected value of the expression</param>
		static void RunTest(string expression, long? expectedResult)
		{
			using TokenReader reader = new(expression);

			List<Token> tokens = [];
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			long? result;
			try
			{
				result = PreprocessorExpression.Evaluate(new PreprocessorTestContext(), tokens);
			}
			catch (PreprocessorException)
			{
				result = null;
			}

			Assert.AreEqual(expectedResult, result);
		}
	}
}
