// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class PreprocessorStateTests
	{
		[TestMethod]
		public void Run()
		{
			PreprocessorState baseState = new();
			baseState.DefineMacro(ParseMacro("FOO", null, "123"));
			baseState.DefineMacro(ParseMacro("BAR", ["Arg1", "Arg2"], "Arg1 + Arg2"));
			Assert.AreEqual("m BAR(Arg1, Arg2)=Arg1 + Arg2\nm FOO=123\n", FormatState(baseState));

			// Modify the state and revert it
			{
				PreprocessorState state = new(baseState);
				Assert.AreEqual(FormatState(baseState), FormatState(state));

				PreprocessorTransform transform1 = state.BeginCapture();
				state.UndefMacro(Identifier.FindOrAdd("FOO"));
				state.DefineMacro(ParseMacro("FOO2", null, "FOO()"));
				state.PushBranch(PreprocessorBranch.Active);
				Assert.AreEqual("b Active\nm BAR(Arg1, Arg2)=Arg1 + Arg2\nm FOO2=FOO()\n", FormatState(state));
				Assert.AreEqual("+b Active\n+m FOO undef\n+m FOO2=FOO()\n", FormatTransform(transform1));

				PreprocessorState state2 = new(baseState);
				state2.TryToApply(transform1);
				Assert.AreEqual(FormatState(state), FormatState(state2));

				PreprocessorTransform transform2 = state.BeginCapture();
				state.PopBranch();
				state.UndefMacro(Identifier.FindOrAdd("FOO"));
				state.DefineMacro(ParseMacro("FOO", null, "123"));
				state.UndefMacro(Identifier.FindOrAdd("FOO2"));
				Assert.AreEqual(FormatState(baseState), FormatState(state));
				Assert.AreEqual("-b Active\n+m FOO=123\n+m FOO2 undef\n", FormatTransform(transform2));

				state2.TryToApply(transform2);
				Assert.AreEqual(FormatState(baseState), FormatState(state2));
			}

			// Check the tracking of branches
			{
				PreprocessorState state = new(baseState);
				PreprocessorTransform transform1 = state.BeginCapture();
				state.IsCurrentBranchActive();
				Assert.AreEqual("=b Active\n", FormatTransform(transform1));

				state.PushBranch(PreprocessorBranch.Active);
				PreprocessorTransform transform2 = state.BeginCapture();
				state.IsCurrentBranchActive();
				state.PopBranch();
				state.EndCapture();
				Assert.AreEqual("-b Active\n", FormatTransform(transform2));

				state.PushBranch(0);
				Assert.AreEqual("False", state.CanApply(transform2).ToString());

				state.PopBranch();
				state.PushBranch(PreprocessorBranch.Active);
				Assert.AreEqual("True", state.CanApply(transform2).ToString());
			}
		}

		static string FormatState(PreprocessorState state)
		{
			StringBuilder result = new();
			foreach (PreprocessorBranch branch in state.CurrentBranches)
			{
				result.AppendFormat("b {0}\n", branch);
			}
			foreach (PreprocessorMacro macro in state.CurrentMacros.OrderBy(x => x.Name))
			{
				result.AppendFormat("m {0}\n", macro.ToString().TrimEnd());
			}
			return result.ToString();
		}

		static string FormatTransform(PreprocessorTransform transform)
		{
			StringBuilder result = new();
			if (transform.RequireTopmostActive.HasValue)
			{
				result.AppendFormat("=b {0}\n", transform.RequireTopmostActive.Value ? "Active" : "0");
			}
			foreach (PreprocessorBranch branch in transform.RequiredBranches)
			{
				result.AppendFormat("-b {0}\n", branch);
			}
			foreach (PreprocessorBranch branch in transform.NewBranches)
			{
				result.AppendFormat("+b {0}\n", branch);
			}
			foreach (KeyValuePair<Identifier, PreprocessorMacro?> macro in transform.RequiredMacros)
			{
				if (macro.Value == null)
				{
					result.AppendFormat("=m {0} undef\n", macro.Key);
				}
				else
				{
					result.AppendFormat("=m {0}\n", macro.Value.ToString().TrimEnd());
				}
			}
			foreach (KeyValuePair<Identifier, PreprocessorMacro?> macro in transform.NewMacros)
			{
				if (macro.Value == null)
				{
					result.AppendFormat("+m {0} undef\n", macro.Key);
				}
				else
				{
					result.AppendFormat("+m {0}\n", macro.Value.ToString().TrimEnd());
				}
			}
			return result.ToString();
		}

		static PreprocessorMacro ParseMacro(string name, List<string>? parameters, string value)
		{
			List<Token> tokens = [];

			using TokenReader reader = new(value);
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			return parameters == null
				? new PreprocessorMacro(Identifier.FindOrAdd(name), null, tokens)
				: new PreprocessorMacro(Identifier.FindOrAdd(name), [.. parameters.Select(Identifier.FindOrAdd)], tokens);
		}
	}
}
