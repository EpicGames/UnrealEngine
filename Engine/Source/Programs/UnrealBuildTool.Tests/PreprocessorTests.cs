// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Text;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildBase;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class PreprocessorTests
	{
		[TestMethod]
		public void BasicTests()
		{
			RunTest("token", "token\n");
			RunTest("#", "");
		}

		[TestMethod]
		public void ObjectMacroTests()
		{
			RunTest("#define X token\nX", "token\n");
			RunTest("#define A B C D\nA", "B C D\n");
			RunTest("#define A B ## D\nA", "BD\n");

			RunTest("#define X list of tokens\nX", "list of tokens\n");
			RunTest("#define LST list\n#define TOKS tokens\n#define LOTS LST of TOKS\nLOTS LOTS", "list of tokens list of tokens\n");
			RunTest("#define LOTS LST of TOKS\n#define LST list\n#define TOKS tokens\nLOTS LOTS", "list of tokens list of tokens\n");

			RunTest("#define A MACRO\nA", "MACRO\n");
			RunTest("#define A MACRO\n#undef A\nA", "A\n");
		}

		[TestMethod]
		public void FunctionMacroTests()
		{
			RunTest("#define X(x) token x other\nX(and one).", "token and one other.\n");
			RunTest("#define X(x,y) token x and y\nX(1, 2).", "token 1 and 2.\n");
			RunTest("#define INC(x) (x + 2)\nINC", "INC\n");
			RunTest("#define TEST(x) x\\\n?\nTEST(A)", "A?\n");

			RunTest("#define F(A) <A>\nF(x)", "<x>\n");
			RunTest("#define F(A,B) <A,B>\nF(x,y) + 1", "<x,y> + 1\n");
			RunTest("#define F(A,B,C) <A,B,C>\nF(x,y,z)", "<x,y,z>\n");
			RunTest("#define F(...) <__VA_ARGS__>\nF(x)", "<x>\n");
			RunTest("#define F(...) <__VA_ARGS__>\nF(x,y)", "<x,y>\n");
			RunTest("#define F(A,...) <A,__VA_ARGS__>\nF(x,y)", "<x,y>\n");
			RunTest("#define F(A,...) <A, __VA_ARGS__>\nF(x,y)", "<x, y>\n");
			RunTest("#define F(A,...) <A, __VA_ARGS__>\nF(x,y, z)", "<x, y, z>\n");

			RunTest("#define FUNC(x) arg=x.\nFUNC(var) FUNC(2)", "arg=var. arg=2.\n");
			RunTest("#define FUNC(x,y,z) int n = z+y*x;\nFUNC(1,2,3)", "int n = 3+2*1;\n");
			RunTest("#define X 20\n#define FUNC(x,y) x+y\nx=FUNC(X,Y);", "x=20+Y;\n");
			RunTest("#define FA(x,y) FB(x,y)\n#define FB(x,y) x + y\nFB(1,2);", "1 + 2;\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF()", "printf()\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF(\"hello\")", "printf(\"hello\")\n");
			RunTest("#define PRINTF(...) printf(__VA_ARGS__)\nPRINTF(\"%d\", 1)", "printf(\"%d\", 1)\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test\")", "printf(\"test\", )\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test %s\", \"hello\")", "printf(\"test %s\", \"hello\")\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, __VA_ARGS__)\nPRINTF(\"test %s %d\", \"hello\", 1)", "printf(\"test %s %d\", \"hello\", 1)\n");
			RunTest("#define PRINTF(FORMAT, ...) printf(FORMAT, ## __VA_ARGS__)\nPRINTF(\"test\")", "printf(\"test\" )\n");

			RunTest("#define INC(x) (x + 2)\nINC(\n1\n)", "(1 + 2)\n");

			RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(+=)", "\"+=\"\n");
			RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(:>)", "\":>\"\n");
			RunTest("#define STRINGIZE(ARG) #ARG\nSTRINGIZE(3.1415)", "\"3.1415\"\n");

			RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(+, =)", "+=\n");
			RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(3, .14159)", "3.14159\n");
			RunTest("#define CONCAT(X, Y) X ## Y\nCONCAT(3, hello)", "3hello\n");
			RunTest("#define CONCAT(X, Y) X ## #Y\nCONCAT(u, hello)", "u\"hello\"\n");
			RunTest("#define CONCAT(X, ...) X ## __VA_ARGS__\nCONCAT(hello) there", "hello there\n");
			RunTest("#define CONCAT(X, ...) X ## __VA_ARGS__\nCONCAT(hello, there)", "hellothere\n");

			RunTest("#define A(X) MACRO X\nA(x)", "MACRO x\n");
			RunTest("#define A(X) MACRO X\n#undef A\nA(x)", "A(x)\n");
		}

		[TestMethod]
		public void ConditionalTests()
		{
			RunTest("#if 1\nfirst_branch\n#endif\nend", "first_branch\nend\n");
			RunTest("#if 0\nfirst_branch\n#endif\nend", "end\n");
			RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#endif", "branch_1\n");
			RunTest("#if 0\nbranch_1\n#else\nbranch_2\n#endif", "branch_2\n");
			RunTest("#define A\n#ifdef A\nYes\n#endif", "Yes\n");
			RunTest("#define B\n#ifdef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define A\n#ifndef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define B\n#ifndef A\nYes\n#endif", "Yes\n");
			RunTest("#define A\n#undef A\n#ifdef A\nYes\n#endif\nNo", "No\n");
			RunTest("#define A\n#undef A\n#ifndef A\nYes\n#endif", "Yes\n");
			RunTest("#define A\n#ifdef A\nYes\n#else\nNo\n#endif", "Yes\n");
			RunTest("#define B\n#ifdef A\nYes\n#else\nNo\n#endif", "No\n");
		}

		[TestMethod]
		public void ExpressionTests()
		{
			RunTest("#if 1 + 2 > 3\nYes\n#endif\nNo", "No\n");
			RunTest("#if 1 + 2 >= 3\nYes\n#endif", "Yes\n");
			RunTest("#define ONE 1\n#define TWO 2\n#define PLUS(x, y) x + y\n#if PLUS(ONE, TWO) > 3\nYes\n#endif\nNo", "No\n");
			RunTest("#define ONE 1\n#define TWO 2\n#define PLUS(x, y) x + y\n#if PLUS(ONE, TWO) >= 3\nYes\n#endif", "Yes\n");
			RunTest("#define ONE 1\n#if defined ONE\nOne\n#elif defined TWO\nTwo\n#else\nThree\n#endif", "One\n");
			RunTest("#define TWO 1\n#if defined ONE\nOne\n#elif defined TWO\nTwo\n#else\nThree\n#endif", "Two\n");
			RunTest("#define ONE 0\n#if defined(ONE) + defined(TWO) >= 1\nYes\n#else\nNo\n#endif", "Yes\n");
			RunTest("#define ONE 0\n#if defined(ONE) + defined(TWO) >= 2\nYes\n#else\nNo\n#endif", "No\n");
			RunTest("#define ONE 0\n#define TWO\n#if defined(ONE) + defined(TWO) >= 2\nYes\n#else\nNo\n#endif", "Yes\n");
		}

		[TestMethod]
		public void ConcatenationTests()
		{
			RunTest("#define FUNC(x) Value = x\n#define PP_JOIN(x, y) x ## y\n#define RESULT(x) PP_JOIN(FU, NC)(x)\nRESULT(1234)", "Value = 1234\n");
			RunTest("#define VALUE 1234\n#define PP_JOIN(x, y) x ## y\n#define RESULT PP_JOIN(V, ALUE)\nRESULT", "1234\n");
			RunTest("#define V 1\n#define ALUE 2\n#define VALUE 1234\n#define PP_JOIN(x, y) x ## y\n#define RESULT PP_JOIN(V, ALUE)\nRESULT", "1234\n");
		}

		[TestMethod]
		public void VarargTests()
		{
			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a)", "(a )\n");
			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a,b)", "(a,b)\n");
			RunTest("#define FUNC(fmt,...) (fmt, ## __VA_ARGS__)\nFUNC(a,b )", "(a,b)\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a)", "(a )\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a,b)", "(a,b)\n");
			RunTest("#define FUNC(fmt, ...) (fmt, ## __VA_ARGS__)\nFUNC(a,b )", "(a,b)\n");
		}

		[TestMethod]
		public void EmptyTokenTests()
		{
			RunTest("#define EMPTY_TOKEN\n#define FUNC(_FuncName, _HType1, _HArg1) _FuncName(_HType1 _HArg1);\nFUNC(hello, EMPTY_TOKEN, int)", "hello( int);\n");

			RunTest("#define EMPTY_TOKEN\n#define GCC_EXTENSION(...) 123 , ## __VA_ARGS__\nGCC_EXTENSION(EMPTY_TOKEN)", "123  \n");

			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x)\nFUNC(EMPTY_TOKEN A)", "( A)\n");
			RunTest("#define EMPTY_TOKEN\n#define FUNC(x,y) (x y)\nFUNC(EMPTY_TOKEN,A)", "( A)\n");
			//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x)\n#define FUNC_2(x,y) FUNC(x y)\nFUNC_2(EMPTY_TOKEN,A)", "( A)\n");
			//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x) (x EMPTY_TOKEN)\nFUNC(A)", "(A )\n");
			//			RunTest("#define EMPTY_TOKEN\n#define FUNC(x,y) (x y)\nFUNC(A,)", "(A)\n");

			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    EMPTY    y)", "(x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(x    EMPTY    y    EMPTY)", "(x  y )\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(EMPTY    x    EMPTY    y)", "( x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(    EMPTY    x    EMPTY    y)", "( x  y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\nFUNC(EMPTY    x    EMPTY    y    )", "( x  y)\n");

			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    EMPTY    y)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(x    EMPTY    y    EMPTY)", "(x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(EMPTY    x    EMPTY    y)", "( x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(    EMPTY    x    EMPTY    y)", "( x y)\n");
			RunTest("#define EMPTY\n#define FUNC(x) (x)\n#define FUNC_2(x) FUNC(x)\nFUNC_2(EMPTY    x    EMPTY    y    )", "( x y)\n");

			RunTest("#define EMPTY\n#define FUNC(x) ( x )\nFUNC(EMPTY EMPTY)", "(   )\n");
			RunTest("#define EMPTY\n#define FUNC(x) ( x)\nFUNC(EMPTY EMPTY x)", "(   x)\n");
			RunTest("#define EMPTY\n#define FUNC(x,y) ( x y)\n#define FUNC2(x,y) FUNC(x,y)\nFUNC2(EMPTY EMPTY EMPTY, x)", "(   x)\n");
			RunTest("#define EMPTY\n#define DOUBLE_EMPTY EMPTY EMPTY\n#define FUNC(x) ( x)\nFUNC(DOUBLE_EMPTY x)", "(   x)\n");
			RunTest("#define EMPTY\n#define DOUBLE_EMPTY EMPTY EMPTY\n#define FUNC(x,y) ( x y )\n#define FUNC2(x,y) FUNC(x,y)\nFUNC2(DOUBLE_EMPTY EMPTY, x)", "(   x )\n");
			RunTest("#define EMPTY\n#define FUNC(x,y) ( x y )\nFUNC(EMPTY EMPTY EMPTY EMPTY EMPTY EMPTY, x)", "(       x )\n");
		}

		[TestMethod]
		public void MiscTests()
		{
			RunTest("#define REGISTER_NAME(num,name) NAME_##name = num,\nREGISTER_NAME(201, TRUE)", "NAME_TRUE = 201,\n");
			RunTest("#define FUNC_2(X) VALUE=X\n#define FUNC_N(X) FUNC_##X\n#define OBJECT FUNC_N(2)\nOBJECT(1234)", "VALUE=1234\n");
			RunTest("#define GCC_EXTENSION(...) 123, ## __VA_ARGS__\nGCC_EXTENSION(456)", "123,456\n");

			RunTest("#define _NON_MEMBER_CALL(FUNC, CV_REF_OPT) FUNC(X,CV_REF_OPT)\n#define _NON_MEMBER_CALL_CV(FUNC, REF_OPT) _NON_MEMBER_CALL(FUNC, REF_OPT) _NON_MEMBER_CALL(FUNC, const REF_OPT)\n#define _NON_MEMBER_CALL_CV_REF(FUNC) _NON_MEMBER_CALL_CV(FUNC, ) _NON_MEMBER_CALL_CV(FUNC, &)\n#define _IS_FUNCTION(X,CV_REF_OPT) (CV_REF_OPT)\n_NON_MEMBER_CALL_CV_REF(_IS_FUNCTION)", "() (const) (&) (const &)\n");

			RunTest("#define TEXT(x) L ## x\n#define checkf(expr, format, ...) FDebug::AssertFailed(#expr, format, ##__VA_ARGS__)\ncheckf( true, TEXT( \"hello world\" ) );", "FDebug::AssertFailed(\"true\", L\"hello world\" );\n");
		}

		[TestMethod]
		public void ErrorTests()
		{
			RunTest("#define", null);
			RunTest("#define INC(x) (x + 2)\nINC(", null);
			RunTest("#define +", null);
			RunTest("#define A B __VA_ARGS__ D\nA", null);
			RunTest("#define A ## B\nA", null);
			RunTest("#define A B ##\nA", null);
			RunTest("#define defined not_defined", null);

			RunTest("#define F(A) <A>\nF(x,y) + 1", null);
			RunTest("#define F(A,) <A>\nF(x)", null);
			RunTest("#define F(A+) <A>\nF(x)", null);
			RunTest("#define F(A,A) <A>\nF(x,y)", null);
			RunTest("#define F(A,B) <A,B>\nF(x) + 1", null);
			RunTest("#define F(A,B,) <A,B>\nF(x,y) + 1", null);
			//			RunTest("#define F(...) <__VA_ARGS__>\nF(x,)", null);
			RunTest("#define F(A...) <A, __VA_ARGS__>\nF(x)", null);
			RunTest("#define F(A...) <A, __VA_ARGS__>\nF(x,y)", null);
			RunTest("#define F(A,__VA_ARGS__) <A, __VA_ARGS__>\nF(x,y)", null);
			RunTest("#define F(A) #+\nF(x)", null);
			RunTest("#define F(A) #B\nF(x)", null);
			RunTest("#define F(A) ## A\nF(x)", null);
			RunTest("#define F(A) A ##\nF(x)", null);
			RunTest("#define F(A) <__VA_ARGS__>\nF(x)", null);

			RunTest("#define INC(x\n)", null);
			RunTest("#if 1\nbranch_1\n#else garbage\nbranch_2\n#endif\nend", null);
			//			RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#endif garbage\nend", null);
			RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#elif 1\nbranch_3\n#endif\nend", null);
			RunTest("#if 1\nbranch_1\n#else\nbranch_2\n#else\nbranch_3\n#endif", null);
			RunTest("#if 0\nbranch_1\n#else\nbranch_2\n#else\nbranch_3\n#endif", null);
			RunTest("#elif\nbranch_1\n#else\nbranch_2\n#endif\nend", null);
			RunTest("#ifdef +\nbranch_1\n#else\nbranch_2\n#endif", null);
			RunTest("#ifdef A 1\nbranch_1\n#else\nbranch_2\n#endif", null);
			RunTest("#define A()\n#if A(\n)\nOK\n#endif", null);
			RunTest("#define A MACRO\n#undef A B\nA", null);
		}

		/// <summary>
		/// Preprocess a fragment of code, and check it results in an expected sequence of tokens
		/// </summary>
		/// <param name="fragment">The code fragment to preprocess</param>
		/// <param name="expectedResult">The expected sequence of tokens, as a string. Null to indicate that the input is invalid, and an exception is expected.</param>
		static void RunTest(string fragment, string? expectedResult)
		{
			string[] lines = fragment.Split('\n');

			SourceFile file = new(FileItem.GetItemByPath(Path.Combine(Path.GetTempPath(), "temp.cpp")), TokenReader.GetNullTerminatedByteArray(fragment));

			Preprocessor instance = new();

			string? result;
			try
			{
				PreprocessorFileContext context = new(file, null);
				List<Token> outputTokens = [];
				for (int markupIdx = 0; markupIdx < file.Markup.Length; markupIdx++)
				{
					SourceFileMarkup markup = file.Markup[markupIdx];
					if (markup.Type == SourceFileMarkupType.Text)
					{
						if (instance.IsCurrentBranchActive())
						{
							StringBuilder sourceText = new();

							int lastLineIndex = (markupIdx + 1 < file.Markup.Length) ? file.Markup[markupIdx + 1].LineNumber - 1 : lines.Length;
							for (int lineIndex = markup.LineNumber - 1; lineIndex < lastLineIndex; lineIndex++)
							{
								sourceText.AppendLine(lines[lineIndex]);
							}

							List<Token> tokens = [];

							using TokenReader reader = new(sourceText.ToString());
							while (reader.MoveNext())
							{
								tokens.Add(reader.Current);
							}

							instance.ExpandMacros(tokens, outputTokens, false, context);
						}
					}
					else
					{
						instance.ParseMarkup(markup.Type, markup.Tokens ?? [], context);
					}
				}
				result = Token.Format(outputTokens);
			}
			catch (PreprocessorException)
			{
				result = null;
			}
			Assert.AreEqual(expectedResult, result);
		}
	}
}
