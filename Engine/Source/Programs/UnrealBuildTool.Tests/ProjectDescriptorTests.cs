// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildBase;
using UnrealBuildTool.Tests.TestUtilities;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class ProjectDescriptorTests
	{
		[TestMethod]
		public void TestLoadingProjectWithInvalidModuleAllowListFails()
		{
			string uprojectPath = Path.Combine(TestDirs.Stubs.FullName, "UprojectStubs", "BadModuleAllowList.uproject.ubttest");

			using SafeTestDirectory testRootDirectory = TestDirs.Create();
			using SafeTestFile uprojectFile = new(File.ReadAllText(uprojectPath), "TestStub.uproject", testRootDirectory.TemporaryDirectory);

			CheckExceptionContainsError(uprojectFile, "Editro", "TargetAllowList");
		}

		[TestMethod]
		public void TestLoadingProjectWithInvalidPluginAllowListFails()
		{
			string uprojectPath = Path.Combine(TestDirs.Stubs.FullName, "UprojectStubs", "BadPluginAllowList.uproject.ubttest");

			using SafeTestDirectory testRootDirectory = TestDirs.Create();
			using SafeTestFile uprojectFile = new(File.ReadAllText(uprojectPath), "TestStub.uproject", testRootDirectory.TemporaryDirectory);

			CheckExceptionContainsError(uprojectFile, "Shpipping", "TargetConfigurationAllowList");
		}

		private static void CheckExceptionContainsError(SafeTestFile uprojectFile, params string[] requiredTexts)
		{
			Exception ex = Assert.ThrowsExactly<Exception>(() => ProjectDescriptor.FromFile(uprojectFile));
			Assert.IsNotNull(ex.InnerException);
			Assert.AreEqual(typeof(BuildLogEventException), ex.InnerException.GetType());
			BuildLogEventException inner = (BuildLogEventException)ex.InnerException;

			foreach (string text in requiredTexts)
			{
				StringAssert.Contains(inner.Message, text, StringComparison.Ordinal);
			}
		}
	}
}
