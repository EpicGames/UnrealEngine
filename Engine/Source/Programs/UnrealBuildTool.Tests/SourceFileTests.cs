// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool.Tests
{
	/// <summary>
	/// Tests for reading source file markup
	/// </summary>
	[TestClass]
	public class SourceFileTests
	{
		[TestMethod]
		public void Run()
		{
			List<DirectoryReference> baseDirectories =
			[
				DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Runtime"),
				DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Developer"),
				DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Editor"),
			];

			foreach (FileReference pluginFile in PluginsBase.EnumeratePlugins((FileReference?)null))
			{
				DirectoryReference pluginSourceDir = DirectoryReference.Combine(pluginFile.Directory, "Source");
				if (DirectoryReference.Exists(pluginSourceDir))
				{
					baseDirectories.Add(pluginSourceDir);
				}
			}

			ConcurrentBag<SourceFile> sourceFiles = [];
			using (GlobalTracer.Instance.BuildSpan("Scanning source files").StartActive())
			{
				using (ThreadPoolWorkQueue queue = new())
				{
					foreach (DirectoryReference baseDirectory in baseDirectories)
					{
						queue.Enqueue(() => ParseSourceFiles(DirectoryItem.GetItemByDirectoryReference(baseDirectory), sourceFiles, queue));
					}
				}
			}

			FileReference tempDataFile = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Temp", "SourceFileTests.bin");
			DirectoryReference.CreateDirectory(tempDataFile.Directory);

			using (GlobalTracer.Instance.BuildSpan("Writing source file data").StartActive())
			{
				using (BinaryArchiveWriter writer = new(tempDataFile))
				{
					writer.WriteList(sourceFiles.ToList(), x => x.Write(writer));
				}
			}

			List<SourceFile>? readSourceFiles = [];
			using (GlobalTracer.Instance.BuildSpan("Reading source file data").StartActive())
			{
				using (BinaryArchiveReader reader = new(tempDataFile))
				{
					readSourceFiles = reader.ReadList(() => new SourceFile(reader));
				}
			}
		}

		static void ParseSourceFiles(DirectoryItem directory, ConcurrentBag<SourceFile> sourceFiles, ThreadPoolWorkQueue queue)
		{
			foreach (DirectoryItem subDirectory in directory.EnumerateDirectories())
			{
				queue.Enqueue(() => ParseSourceFiles(subDirectory, sourceFiles, queue));
			}

			foreach (FileItem file in directory.EnumerateFiles())
			{
				if (file.HasExtension(".h") || file.HasExtension(".cpp"))
				{
					queue.Enqueue(() => sourceFiles.Add(new SourceFile(file)));
				}
			}
		}
	}
}
