// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildTool.Tests.TestUtilities;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class TargetRulesTests
	{
		[TestMethod]
		public void TestTargetRequiresUniqueEnvironmentFirstOrder()
		{
			DirectoryReference testStubsDirectory = TestDirs.Stubs;

			// Generate ephemeral TargetRules & uproject files
			using (SafeTestDirectory testRootDirectory = TestDirs.Create())
			using (SafeTestDirectory testSourceDirectory = SafeTestDirectory.CreateTestDirectory(Path.Combine(testRootDirectory.TemporaryDirectory, "Source")))
			using (SafeTestFile testEditorTargetTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "TargetStubs", "TestStubEditor.Target.ubttest")), "TestStubEditor.Target.cs", testSourceDirectory.TemporaryDirectory))
			using (SafeTestFile testUprojectTestFile = new(File.ReadAllText(Path.Combine(testStubsDirectory.FullName, "UprojectStubs", "TestStub.uproject.ubttest")), "TestStub.uproject", testRootDirectory.TemporaryDirectory))
			{
				FileReference testUproject = new(testUprojectTestFile.TemporaryFile);

				// Generate rules assembly & subsequent targets
				RulesAssembly generatedTestRulesAssembly = RulesCompiler.CreateTargetRulesAssembly(testUproject, "TestStubEditor", false, false, false, null, false, new TestLogger());
				TargetRules testTargetRules = generatedTestRulesAssembly.CreateTargetRules("TestStubEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, null, testUproject, null, new TestLogger());
				string? baseTarget = null;

				// Do not mutate the object
				{
					bool result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsFalse(result);
				}

				// Mutate the object against a RequiresUniqueEnvironment
				{
					WarningLevel preWarningLevel = testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel;
					testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel = WarningLevel.Warning;
					bool result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsTrue(result);

					testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel = preWarningLevel;
				}

				// Mutate the object against a RequiresUniqueEnvironment (Clang only)
				{
					WarningLevel preWarningLevel = testTargetRules.CppCompileWarningSettings.NonTrivialMemAccessWarningLevel;
					testTargetRules.CppCompileWarningSettings.NonTrivialMemAccessWarningLevel = WarningLevel.Warning;
					bool result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsTrue(result);

					testTargetRules.CppCompileWarningSettings.NonTrivialMemAccessWarningLevel = preWarningLevel;
				}

				// Mutate the object against a RequiresUniqueEnvironment (Default warning set)
				{
					WarningLevel preWarningLevel = testTargetRules.CppCompileWarningSettings.DeprecationWarningLevel;
					testTargetRules.CppCompileWarningSettings.DeprecationWarningLevel = WarningLevel.Error;
					bool result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsTrue(result);

					testTargetRules.CppCompileWarningSettings.DeprecationWarningLevel = WarningLevel.Warning;
					result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsFalse(result);

					testTargetRules.CppCompileWarningSettings.DeprecationWarningLevel = preWarningLevel;
				}

				// Mutate the object against a RequiresUniqueEnvironment, and set bOverrideBuildEnvironment=true
				{
					WarningLevel preWarningLevel = testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel;
					testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel = WarningLevel.Warning;
					testTargetRules.bOverrideBuildEnvironment = true;

					bool result = testTargetRules.RequiresUniqueEnvironment(generatedTestRulesAssembly, null, [], out baseTarget);

					Assert.IsFalse(result);

					testTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel = preWarningLevel;
				}
			}
		}
	}
}