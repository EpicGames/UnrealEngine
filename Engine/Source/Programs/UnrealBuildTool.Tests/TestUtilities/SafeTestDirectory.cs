// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class SafeTestDirectory : IDisposable
	{
		internal string TemporaryDirectory { get; private set; }
		private bool _wasFolderCreated = false;

		private SafeTestDirectory(string temporaryDirectory, bool wasFolderCreated = false)
		{
			TemporaryDirectory = temporaryDirectory;
			_wasFolderCreated = wasFolderCreated;
		}

		internal static SafeTestDirectory CreateTestDirectory(string absoluteDirectoryPath)
		{
			SafeTestDirectory returnDirectory = new(absoluteDirectoryPath);
			if (!Directory.Exists(absoluteDirectoryPath))
			{
				Directory.CreateDirectory(absoluteDirectoryPath);
				returnDirectory._wasFolderCreated = true;
			}

			return returnDirectory;
		}

		public void Dispose()
		{
			if (_wasFolderCreated && !String.IsNullOrEmpty(TemporaryDirectory) && Directory.Exists(TemporaryDirectory))
			{
				try
				{
					Directory.Delete(TemporaryDirectory, true);
				}
				catch (Exception)
				{
					// Swallow exception
				}
			}
		}
	}
}
