// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class SafeTestFile : IDisposable
	{
		internal string TemporaryFile { get; private set; }

		internal SafeTestFile(string content, string outputFileName, string outputFolder)
		{
			try
			{
				TemporaryFile = Path.Combine(outputFolder, outputFileName);

				using (StreamWriter writer = new(TemporaryFile))
				{
					writer.Write(content);
				}
			}
			catch (Exception)
			{
				throw new InvalidOperationException("Unable to create test file. Test not feasible.");
			}
		}

		public void Dispose()
		{
			if (!String.IsNullOrEmpty(TemporaryFile) && File.Exists(TemporaryFile))
			{
				File.Delete(TemporaryFile);
			}
		}

		public static implicit operator FileReference(SafeTestFile file)
			=> new(file.TemporaryFile);
	}
}
