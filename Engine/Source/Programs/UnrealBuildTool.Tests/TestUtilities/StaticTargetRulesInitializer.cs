// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests.TestUtilities
{
	[TestClass]
	public static class StaticTargetRulesInitializer
	{
		[AssemblyInitialize]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Required for UnitTesting")]
		public static void InitializeBuildEnvironment(TestContext context)
		{
			// Clean any previous runs linger temp folder
			// Remarks: this is due to target rules modules dll being held by the test process; full cleanup not feasible on existing run
			string[] subfolders = Directory.GetDirectories(TestDirs.UBTTestFolderRoot, $"{TestDirs.UBTTestFolderPrefix}*", SearchOption.TopDirectoryOnly);

			foreach (string folder in subfolders)
			{
				if (Directory.Exists(folder))
				{
					try
					{
						Directory.Delete(folder, true);
					}
					catch (Exception) { }
				}
			}

			// Example, on how to register a fully stubbed test environment.
			/*
			 	UEBuildPlatform.RegisterBuildPlatform(
					new TestBuildPlatfrom(
							UnrealTargetPlatform.Win64,
							new TestSDK(logger),
							new UnrealArchitectureConfig(UnrealArch.X64),
							logger),
					logger
			);*/

			ILogger logger = new TestLogger();

			UEBuildPlatform? _;
			if (!UEBuildPlatform.TryGetBuildPlatform(UnrealTargetPlatform.Win64, out _))
			{
				MicrosoftPlatformSDK sdk = new(logger);
				UEBuildPlatform.RegisterBuildPlatform(new WindowsPlatform(UnrealTargetPlatform.Win64, sdk, logger), logger);
			}

			if (!UEBuildPlatform.TryGetBuildPlatform(UnrealTargetPlatform.Android, out _))
			{
				AndroidPlatformSDK asdk = new(logger);
				UEBuildPlatform.RegisterBuildPlatform(new AndroidPlatform(asdk, logger), logger);
			}

			XmlConfig.ReadConfigFiles(null, null, logger);
		}
	}
}
