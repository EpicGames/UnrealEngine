// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class TestBuildPlatfrom : UEBuildPlatform
	{
		public TestBuildPlatfrom(UnrealTargetPlatform inPlatform, UEBuildPlatformSDK sdk, UnrealArchitectureConfig architectureConfig, ILogger inLogger) : base(inPlatform, sdk, architectureConfig, inLogger)
		{

		}
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules target)
		{
			throw new System.NotImplementedException();
		}

		public override void Deploy(TargetReceipt receipt)
		{

		}

		public override bool IsBuildProduct(string fileName, string[] namePrefixes, string[] nameSuffixes)
		{
			return false;
		}

		public override void SetUpEnvironment(ReadOnlyTargetRules target, CppCompileEnvironment compileEnvironment, LinkEnvironment linkEnvironment)
		{

		}

		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules target)
		{
			return false;
		}
	}
}
