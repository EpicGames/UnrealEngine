// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class TestSDK : UEBuildPlatformSDK
	{
		public TestSDK(ILogger inLogger) : base(inLogger)
		{
		}

		public override bool TryConvertVersionToInt(string? stringValue, out ulong outValue, string? hint = null)
		{
			outValue = 1;
			return true;
		}

		protected override string? GetInstalledSDKVersion()
		{
			return "1.0.0";
		}
	}
}
