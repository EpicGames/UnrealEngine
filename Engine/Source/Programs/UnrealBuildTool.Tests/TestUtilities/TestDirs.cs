// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class TestDirs
	{
		internal static DirectoryReference Stubs
		{
			get
			{
				DirectoryReference programsDirectory = DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Programs");
				return DirectoryReference.Combine(programsDirectory, "UnrealBuildTool.Tests", "UBT");
			}
		}

		internal const string UBTTestFolderPrefix = "UBT-Test-";
		internal static readonly string UBTTestFolderRoot = Path.GetTempPath();

		internal static SafeTestDirectory Create()
		{
			string tempDirName = $"{UBTTestFolderPrefix}{Guid.NewGuid()}";
			string tempDirPath = Path.Combine(UBTTestFolderRoot, tempDirName);
			return SafeTestDirectory.CreateTestDirectory(tempDirPath);
		}
	}
}
