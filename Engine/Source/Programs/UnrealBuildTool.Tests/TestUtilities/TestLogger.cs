// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class TestLogger : ILogger
	{
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
		{
			return null;
		}

		public bool IsEnabled(LogLevel logLevel)
		{
			return false;
		}

		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
		}
	}
}