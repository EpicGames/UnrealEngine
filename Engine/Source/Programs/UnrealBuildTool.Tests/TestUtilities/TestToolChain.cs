// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool.Tests.TestUtilities
{
	internal class TestToolChain : UEToolChain
	{
		public TestToolChain(ILogger inLogger) : base(inLogger)
		{
		}

		public override FileItem? LinkFiles(LinkEnvironment linkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder graph)
		{
			return null;
		}

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment compileEnvironment, IEnumerable<FileItem> inputFiles, DirectoryReference outputDir, string moduleName, IActionGraphBuilder graph)
		{
			return null!;
		}

		protected override CPPOutput CompileISPCFiles(CppCompileEnvironment compileEnvironment, IEnumerable<FileItem> inputFiles, DirectoryReference outputDir, IActionGraphBuilder graph)
		{
			return null!;
		}

		protected override CPPOutput GenerateISPCHeaders(CppCompileEnvironment compileEnvironment, IEnumerable<FileItem> inputFiles, DirectoryReference outputDir, IActionGraphBuilder graph)
		{
			return null!;
		}
	}
}
