// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class TokenReaderTests
	{
		[TestMethod]
		public void Run()
		{
			RunLexerTest("ABC//hello\nDEF", "Identifier(ABC), Newline, Identifier(DEF)");
			RunLexerTest("ABC/*hello\nworld*/DEF", "Identifier(ABC), Identifier(DEF)");
			RunLexerTest("ABC//hello\r\nDEF", "Identifier(ABC), Newline, Identifier(DEF)");
			RunLexerTest("ABC123", "Identifier(ABC123)");
			RunLexerTest("123ABC", "Number(123ABC)");
			RunLexerTest("FOO(1,2,3,...)", "Identifier(FOO), LeftParen, Number(1), Comma, Number(2), Comma, Number(3), Comma, Ellipsis, RightParen");
		}

		[TestMethod]
		public void SimpleTokens()
		{
			RunLexerTest("=<><<+", "Equals, CompareLess, CompareGreater, LeftShift, Plus");
			RunLexerTest("+=+++++", "Plus, Equals, Plus, Plus, Plus, Plus, Plus");
			RunLexerTest("/+++=", "Divide, Plus, Plus, Plus, Equals");
			RunLexerTest("###", "HashHash, Hash");
			RunLexerTest("():?", "LeftParen, RightParen, Colon, QuestionMark");
			RunLexerTest("..*->->*", "Dot, Dot, Multiply, Minus, CompareGreater, Minus, CompareGreater, Multiply");
			RunLexerTest("+-*/%", "Plus, Minus, Multiply, Divide, Modulo");
			RunLexerTest("|^&~", "BitwiseOr, BitwiseXor, BitwiseAnd, BitwiseNot");
			RunLexerTest("+=-=*=/=", "Plus, Equals, Minus, Equals, Multiply, Equals, Divide, Equals");
			RunLexerTest("%=|=^=", "Modulo, Equals, BitwiseOr, Equals, BitwiseXor, Equals");
			RunLexerTest("&&||!", "LogicalAnd, LogicalOr, LogicalNot");
			RunLexerTest("=<><=>=", "Equals, CompareLess, CompareGreater, CompareLessOrEqual, CompareGreaterOrEqual");
			RunLexerTest("==!=", "CompareEqual, CompareNotEqual");
			RunLexerTest(".....", "Ellipsis, Dot, Dot");
		}

		[TestMethod]
		public void IntegerLiterals()
		{
			RunLexerTest("123+", "Number(123), Plus");
			RunLexerTest("01234567+", "Number(01234567), Plus");
			RunLexerTest("0x123456789ABCDEFabcdef+", "Number(0x123456789ABCDEFabcdef), Plus");
			RunLexerTest("123Ull+", "Number(123Ull), Plus");
			RunLexerTest("123_hello+", "Number(123_hello), Plus");
			RunLexerTest("01234567_hello+", "Number(01234567_hello), Plus");
			RunLexerTest("0x123456789ABCDEFabcdef_hello+", "Number(0x123456789ABCDEFabcdef_hello), Plus");
		}

		[TestMethod]
		public void FloatLiterals()
		{
			RunLexerTest("123.+", "Number(123.), Plus");
			RunLexerTest("123e123+", "Number(123e123), Plus");
			RunLexerTest("123e+123+", "Number(123e+123), Plus");
			RunLexerTest(".123+", "Number(.123), Plus");
			RunLexerTest(".123e123+", "Number(.123e123), Plus");
			RunLexerTest(".123e+123+", "Number(.123e+123), Plus");
			RunLexerTest("0.123+", "Number(0.123), Plus");
			RunLexerTest("0.123e123+", "Number(0.123e123), Plus");
			RunLexerTest("0.123e+123+", "Number(0.123e+123), Plus");
			RunLexerTest("123.123+", "Number(123.123), Plus");
			RunLexerTest("123.123e123+", "Number(123.123e123), Plus");
			RunLexerTest("123.123e+123+", "Number(123.123e+123), Plus");
			RunLexerTest("1.f+", "Number(1.f), Plus");
			RunLexerTest("123.123e+123f+", "Number(123.123e+123f), Plus");
			RunLexerTest("123.123e+123F+", "Number(123.123e+123F), Plus");
		}

		[TestMethod]
		public void CharacterLiterals()
		{
			RunLexerTest("'hello'+", "Character('hello'), Plus");
			RunLexerTest("'hello\\'\\''+", "Character('hello\\'\\''), Plus");
			RunLexerTest("u'hello'+", "Character(u'hello'), Plus");
			RunLexerTest("U'hello'+", "Character(U'hello'), Plus");
			RunLexerTest("L'hello'+", "Character(L'hello'), Plus");
			RunLexerTest("u8'hello'+", "Character(u8'hello'), Plus");
		}

		[TestMethod]
		public void StringLiterals()
		{
			RunLexerTest("\"hello\"+", "String(\"hello\"), Plus");
			RunLexerTest("\"hello\\\"\\\"\"+", "String(\"hello\\\"\\\"\"), Plus");
			RunLexerTest("u\"hello\"+", "String(u\"hello\"), Plus");
			RunLexerTest("U\"hello\"+", "String(U\"hello\"), Plus");
			RunLexerTest("L\"hello\"+", "String(L\"hello\"), Plus");
			RunLexerTest("u8\"hello\"+", "String(u8\"hello\"), Plus");
		}

		[TestMethod]
		public void Identifiers()
		{
			RunLexerTest("abcdefghijklmnopqrstuvwxyz1234567890ABCDEFGHIJKLMNOPQRSTUVWXYZ_+", "Identifier(abcdefghijklmnopqrstuvwxyz1234567890ABCDEFGHIJKLMNOPQRSTUVWXYZ_), Plus");
		}

		[TestMethod]
		public void Comments()
		{
			// Single-line comments
			RunLexerTest("//hello\n+", "Newline, Plus");
			RunLexerTest("//hello\n//world\n+", "Newline, Newline, Plus");

			// Multi-line comments
			RunLexerTest("/*hello*/+", "Plus");
			RunLexerTest("/*/hello*/+", "Plus");
			RunLexerTest("/*/hello**/+", "Plus");
		}

		[TestMethod]
		public void LineNumbers()
		{
			// Test reported line numbers are accurate
			RunLineNumberTest("+", 1);
			RunLineNumberTest("\n+", 2);
			RunLineNumberTest("\r\n+", 2);
			RunLineNumberTest("\r\n\r\n+", 3);
			RunLineNumberTest("\r\n\n+", 3);
			RunLineNumberTest("//comment\n+", 2);
			RunLineNumberTest("//comment\\\n\n+", 3);
			RunLineNumberTest("/*comment\n*/+", 2);
			RunLineNumberTest("/*comment\r\n*/+", 2);
			RunLineNumberTest("/*comment\r\n\r\n*/+", 3);
			RunLineNumberTest("first line \\\n+", 2);
		}

		public static void RunLexerTest(string inputText, string expectedOutputText)
		{
			List<Token> tokens = [];

			using TokenReader reader = new(inputText);
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			string outputText = String.Join(", ", tokens.Select(FormatToken));
			Assert.AreEqual(expectedOutputText, outputText);
		}

		static string FormatToken(Token token)
		{
			StringBuilder result = new();
			result.AppendFormat(token.Type.ToString());
			if (token.Type == TokenType.Identifier || token.Type == TokenType.Number || token.Type == TokenType.Character || token.Type == TokenType.String)
			{
				result.AppendFormat("({0})", token.Text);
			}
			return result.ToString();
		}

		public static void RunLineNumberTest(string inputText, int expectedLineNumber)
		{
			using TokenReader reader = new(inputText);
			while (reader.MoveNext())
			{
				if (reader.Current.Type == TokenType.Plus)
				{
					break;
				}
			}
			Assert.AreEqual(expectedLineNumber, reader.LineNumber);
		}
	}
}
