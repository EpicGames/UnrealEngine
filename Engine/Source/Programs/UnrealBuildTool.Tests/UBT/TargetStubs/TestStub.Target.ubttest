// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

internal class TestStubTarget : TargetRules
{
	public TestStubTarget(TargetInfo target) : base(target)
	{
		Type = TargetType.Game;
		LinkType = TargetLinkType.Monolithic;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
	}
}