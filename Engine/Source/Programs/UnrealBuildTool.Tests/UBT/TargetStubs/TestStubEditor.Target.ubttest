// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

internal class TestStubEditorTarget : TargetRules
{
	public TestStubEditorTarget(TargetInfo target) : base(target)
	{
		Type = TargetType.Editor;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
	}
}