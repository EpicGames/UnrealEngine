{
	"FileVersion": 3,
	"EngineAssociation": "",
	"Category": "",
	"Description": "",
	"Modules": [
		{
			"Name": "NormalModule",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PluginWithBadTargetConfigurationAllowList",
			"Enabled": true,
			"TargetConfigurationAllowList": ["Debug", "Development", "Shpipping"]
		}
	]
}