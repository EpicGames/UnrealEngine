<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <Nullable>enable</Nullable>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
    <!-- We don't require public documentation for test projects -->
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="18.4.0">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestAdapter" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestFramework" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="coverlet.collector" Version="8.0.1">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\UnrealBuildTool\UnrealBuildTool.csproj" />
  </ItemGroup>
</Project>
