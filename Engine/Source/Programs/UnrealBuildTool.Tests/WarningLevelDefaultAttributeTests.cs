// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using UnrealBuildTool.Configuration.CompileWarnings;

namespace UnrealBuildTool.Tests
{
	[TestClass]
	public class WarningLevelDefaultAttributeTests
	{
		internal class TestBuildSettingsProvider : IBuildContextProvider
		{
			internal BuildSettingsVersion _testBuildSettings = BuildSettingsVersion.V1;

			public BuildSettingsVersion GetBuildSettings()
			{
				return _testBuildSettings;
			}
		}

		[TestMethod]
		public void TestWindowsCompletelyDefined()
		{
			List<VersionWarningLevelDefaultAttribute> attributes =
			[
				new VersionWarningLevelDefaultAttribute( WarningLevel.Off, BuildSettingsVersion.V1, BuildSettingsVersion.V2),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Warning, BuildSettingsVersion.V3, BuildSettingsVersion.V4),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Error, BuildSettingsVersion.V5, BuildSettingsVersion.Latest)
			];

			List<VersionWarningLevelDefaultAttribute> merged = VersionWarningLevelDefaultAttribute.EnsureWarningLevelDefaultBounds(attributes);

			Assert.AreEqual(3, merged.Count);
			Assert.AreEqual(BuildSettingsVersion.V1, merged[0].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V2, merged[0].MaxVersion);
			Assert.AreEqual(WarningLevel.Off, merged[0].GetDefaultLevel());

			Assert.AreEqual(BuildSettingsVersion.V3, merged[1].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V4, merged[1].MaxVersion);
			Assert.AreEqual(WarningLevel.Warning, merged[1].GetDefaultLevel());

			Assert.AreEqual(BuildSettingsVersion.V5, merged[2].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.Latest, merged[2].MaxVersion);
			Assert.AreEqual(WarningLevel.Error, merged[2].GetDefaultLevel());
		}

		[TestMethod]
		public void TestGapResolveToLowerBound()
		{
			List<VersionWarningLevelDefaultAttribute> attributes =
			[
				new VersionWarningLevelDefaultAttribute(WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.V2),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Error, BuildSettingsVersion.V4, BuildSettingsVersion.V5),
			];

			List<VersionWarningLevelDefaultAttribute> merged = VersionWarningLevelDefaultAttribute.EnsureWarningLevelDefaultBounds(attributes);

			Assert.AreEqual(2, merged.Count);
			Assert.AreEqual(BuildSettingsVersion.V1, merged[0].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V3, merged[0].MaxVersion);
			Assert.AreEqual(WarningLevel.Warning, merged[0].GetDefaultLevel());

			Assert.AreEqual(BuildSettingsVersion.V4, merged[1].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V5, merged[1].MaxVersion);
			Assert.AreEqual(WarningLevel.Error, merged[1].GetDefaultLevel());
		}

		[TestMethod]
		public void TestLowerUpperOverlapResizeBottomUpward()
		{
			List<VersionWarningLevelDefaultAttribute> attributes =
			[
				new VersionWarningLevelDefaultAttribute(WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.V3),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Error, BuildSettingsVersion.V2, BuildSettingsVersion.V4)
			];

			List<VersionWarningLevelDefaultAttribute> merged = VersionWarningLevelDefaultAttribute.EnsureWarningLevelDefaultBounds(attributes);

			Assert.AreEqual(2, merged.Count);
			Assert.AreEqual(BuildSettingsVersion.V1, merged[0].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V3, merged[0].MaxVersion);
			Assert.AreEqual(WarningLevel.Warning, merged[0].GetDefaultLevel());

			Assert.AreEqual(BuildSettingsVersion.V4, merged[1].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V4, merged[1].MaxVersion);
			Assert.AreEqual(WarningLevel.Error, merged[1].GetDefaultLevel());
		}

		[TestMethod]
		public void TestAllRangeWide()
		{
			List<VersionWarningLevelDefaultAttribute> attributes =
			[
				new VersionWarningLevelDefaultAttribute( WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.V2),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Error, BuildSettingsVersion.V1, BuildSettingsVersion.V4)
			];

			List<VersionWarningLevelDefaultAttribute> merged = VersionWarningLevelDefaultAttribute.EnsureWarningLevelDefaultBounds(attributes);

			Assert.AreEqual(2, merged.Count);

			Assert.AreEqual(BuildSettingsVersion.V1, merged[0].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V2, merged[0].MaxVersion);
			Assert.AreEqual(WarningLevel.Warning, merged[0].GetDefaultLevel());

			Assert.AreEqual(BuildSettingsVersion.V3, merged[1].MinVersion);
			Assert.AreEqual(BuildSettingsVersion.V4, merged[1].MaxVersion);
			Assert.AreEqual(WarningLevel.Error, merged[1].GetDefaultLevel());
		}

		[TestMethod]
		public void TestResolveLowerBound()
		{
			List<WarningLevelDefaultAttribute> attributes =
			[
				new VersionWarningLevelDefaultAttribute(WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.V2),
				new VersionWarningLevelDefaultAttribute(WarningLevel.Error, BuildSettingsVersion.V3, BuildSettingsVersion.V4)
			];
			TestBuildSettingsProvider testBuildSettingsProvider = new();
			BuildSystemContext newBuildSystemContext = new(testBuildSettingsProvider);

			Assert.AreEqual(WarningLevel.Warning, VersionWarningLevelDefaultAttribute.ResolveVersionWarningLevelDefault(attributes, newBuildSystemContext));
			testBuildSettingsProvider._testBuildSettings = BuildSettingsVersion.V2;
			Assert.AreEqual(WarningLevel.Warning, VersionWarningLevelDefaultAttribute.ResolveVersionWarningLevelDefault(attributes, newBuildSystemContext));

			testBuildSettingsProvider._testBuildSettings = BuildSettingsVersion.V3;
			Assert.AreEqual(WarningLevel.Error, VersionWarningLevelDefaultAttribute.ResolveVersionWarningLevelDefault(attributes, newBuildSystemContext));
			testBuildSettingsProvider._testBuildSettings = BuildSettingsVersion.V4;
			Assert.AreEqual(WarningLevel.Error, VersionWarningLevelDefaultAttribute.ResolveVersionWarningLevelDefault(attributes, newBuildSystemContext));

			testBuildSettingsProvider._testBuildSettings = BuildSettingsVersion.V5;
			Assert.AreEqual(WarningLevel.Default, VersionWarningLevelDefaultAttribute.ResolveVersionWarningLevelDefault(attributes, newBuildSystemContext));
		}
	}
}